// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class CommonMenuExtensions : ModuleRules
{
	// TODO: Is this a minimal enough list?
	public CommonMenuExtensions(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.AddRange(
			new string[] {
			}
		);

		PublicIncludePathModuleNames.AddRange(
			new string[] {
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"InputCore",
				"Slate",
				"SlateCore",
				"Engine",
				"UnrealEd", 
				"ToolMenus",
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
			}
		);

	}
}
