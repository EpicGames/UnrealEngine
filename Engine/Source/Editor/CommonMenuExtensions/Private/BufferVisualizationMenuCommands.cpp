// Copyright Epic Games, Inc. All Rights Reserved.

#include "BufferVisualizationMenuCommands.h"

#include "BufferVisualizationData.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EditorViewportClient.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Misc/AssertionMacros.h"
#include "Styling/AppStyle.h"
#include "Templates/Function.h"
#include "UObject/UnrealNames.h"

class UMaterialInterface;

#define LOCTEXT_NAMESPACE "BufferVisualizationMenuCommands"

namespace
{
	struct FMaterialIteratorWithLambda
	{
		typedef TFunction<void(const FString&, const UMaterialInterface*, const FText&)> LambdaFunctionType;
		LambdaFunctionType Lambda;

		inline FMaterialIteratorWithLambda(const LambdaFunctionType& InLambda)
			: Lambda(InLambda)
		{
		}

		inline void ProcessValue(const FString& InMaterialName, const UMaterialInterface* InMaterial, const FText& InDisplayName)
		{
			Lambda(InMaterialName, InMaterial, InDisplayName);
		}
	};

	inline FName NameFromMaterial(const FString& MaterialName)
	{
		return *(FString(TEXT("BufferVisualizationMenu")) + MaterialName);
	}
}

namespace
{
	const FName OVERVIEW_COMMAND_NAME = *(FString(TEXT("BufferVisualizationOverview")));
}

FBufferVisualizationMenuCommands::FBufferVisualizationMenuCommands()
	: TCommands<FBufferVisualizationMenuCommands>
	(
		TEXT("VisualizationMenu"), // Context name for fast lookup
		NSLOCTEXT("Contexts", "VisualizationMenu", "Render Buffer Visualization"), // Localized context name for displaying
		NAME_None, // Parent context name.  
		FAppStyle::GetAppStyleSetName() // Icon Style Set
	),
	CommandMap()
{
}

UE_DEFINE_TCOMMANDS(class FBufferVisualizationMenuCommands)

void FBufferVisualizationMenuCommands::BuildCommandMap()
{
	CommandMap.Empty();

	CreateOverviewCommand();
	CreateVisualizationCommands();
}

void FBufferVisualizationMenuCommands::CreateOverviewCommand()
{
	const FName& CommandName = OVERVIEW_COMMAND_NAME;

	FBufferVisualizationRecord& OverviewRecord = CommandMap.Add(CommandName, FBufferVisualizationRecord());
	OverviewRecord.Name = NAME_None;
	const FText MaterialDisplayDefaultName = FBufferVisualizationData::GetMaterialDefaultDisplayName();
	OverviewRecord.Command = FUICommandInfoDecl(this->AsShared(), CommandName, MaterialDisplayDefaultName, MaterialDisplayDefaultName)
		.UserInterfaceType(EUserInterfaceActionType::RadioButton)
		.DefaultChord(FInputChord());
}

void FBufferVisualizationMenuCommands::CreateVisualizationCommands()
{
	FMaterialIteratorWithLambda Iterator([this](const FString& InMaterialName, const UMaterialInterface* InMaterial, const FText& InDisplayName)
	{
		const FName CommandName = NameFromMaterial(InMaterialName);

		FBufferVisualizationRecord& Record = CommandMap.Add(CommandName, FBufferVisualizationRecord());
		Record.Name = *InMaterialName;
		Record.Command = FUICommandInfoDecl(this->AsShared(), CommandName, InDisplayName, InDisplayName)
			.UserInterfaceType(EUserInterfaceActionType::RadioButton)
			.DefaultChord(FInputChord());
	});

	GetBufferVisualizationData().IterateOverAvailableMaterials(Iterator);
}

void FBufferVisualizationMenuCommands::BuildVisualisationSubMenu(FMenuBuilder& Menu)
{
	const FBufferVisualizationMenuCommands& Commands = FBufferVisualizationMenuCommands::Get();

	Menu.BeginSection("LevelViewportBufferVisualizationMode", LOCTEXT( "BufferVisualizationHeader", "Buffer Visualization Mode" ) );

	Commands.AddOverviewCommandToMenu(Menu);
	Menu.AddMenuSeparator();
	Commands.AddVisualizationCommandsToMenu(Menu);

	Menu.EndSection();
}

void FBufferVisualizationMenuCommands::AddOverviewCommandToMenu(FMenuBuilder& Menu) const
{
	check(CommandMap.Num() > 0);

	Menu.AddMenuEntry(OverviewCommand().Command, NAME_None, FBufferVisualizationData::GetMaterialDefaultDisplayName());
}

void FBufferVisualizationMenuCommands::AddVisualizationCommandsToMenu(FMenuBuilder& Menu) const
{
	check(CommandMap.Num() > 0);

	const TBufferVisualizationModeCommandMap& Commands = CommandMap;

	FMaterialIteratorWithLambda Iterator([&Menu, &Commands](const FString& InMaterialName, const UMaterialInterface* InMaterial, const FText& InDisplayName)
	{
		const FName CommandName = NameFromMaterial(InMaterialName);

		const FBufferVisualizationRecord* Record = Commands.Find(CommandName);
		if (ensureMsgf(Record != NULL, TEXT("Buffer visualization commands don't contain entry [%s]"), *CommandName.ToString()))
		{
			Menu.AddMenuEntry(Record->Command, NAME_None, InDisplayName);
		}
	});

	GetBufferVisualizationData().IterateOverAvailableMaterials(Iterator);
}

FBufferVisualizationMenuCommands::TCommandConstIterator FBufferVisualizationMenuCommands::CreateCommandConstIterator() const
{
	return CommandMap.CreateConstIterator();
}

const FBufferVisualizationMenuCommands::FBufferVisualizationRecord& FBufferVisualizationMenuCommands::OverviewCommand() const
{
	check(CommandMap.Num() > 0);

	return CommandMap.FindChecked(OVERVIEW_COMMAND_NAME);
}

void FBufferVisualizationMenuCommands::RegisterCommands()
{
	BuildCommandMap();
}

void FBufferVisualizationMenuCommands::BindCommands(FUICommandList& CommandList, const TSharedPtr<FEditorViewportClient>& Client) const
{
	// Map Buffer visualization mode actions
	for (FBufferVisualizationMenuCommands::TCommandConstIterator It = FBufferVisualizationMenuCommands::Get().CreateCommandConstIterator(); It; ++It)
	{
		const FBufferVisualizationMenuCommands::FBufferVisualizationRecord& Record = It.Value();
		CommandList.MapAction(
			Record.Command,
			FExecuteAction::CreateStatic(&FBufferVisualizationMenuCommands::ChangeBufferVisualizationMode, Client.ToWeakPtr(), Record.Name ),
			FCanExecuteAction(),
			FIsActionChecked::CreateStatic(&FBufferVisualizationMenuCommands::IsBufferVisualizationModeSelected, Client.ToWeakPtr(), Record.Name ) );
	}
}

void FBufferVisualizationMenuCommands::ChangeBufferVisualizationMode(TWeakPtr<FEditorViewportClient> WeakClient, FName InName)
{
	if (TSharedPtr<FEditorViewportClient> Client = WeakClient.Pin())
	{
		Client->ChangeBufferVisualizationMode(InName);
	}
}

bool FBufferVisualizationMenuCommands::IsBufferVisualizationModeSelected(TWeakPtr<FEditorViewportClient> WeakClient, FName InName)
{
	if (TSharedPtr<FEditorViewportClient> Client = WeakClient.Pin())
	{
		return Client->IsBufferVisualizationModeSelected(InName);
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
