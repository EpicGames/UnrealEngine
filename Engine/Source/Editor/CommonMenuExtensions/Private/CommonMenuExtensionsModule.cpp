// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonMenuExtensionsModule.h"
#include "BufferVisualizationMenuCommands.h"
#include "NaniteVisualizationMenuCommands.h"
#include "LumenVisualizationMenuCommands.h"
#include "MegaLightsVisualizationMenuCommands.h"
#include "SubstrateVisualizationMenuCommands.h"
#include "GroomVisualizationMenuCommands.h"
#include "VirtualShadowMapVisualizationMenuCommands.h"
#include "VirtualTextureVisualizationMenuCommands.h"
#include "RayTracingDebugVisualizationMenuCommands.h"
#include "ShowFlagMenuCommands.h"
#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FCommonMenuExtensionsModule, CommonMenuExtensions);

void FCommonMenuExtensionsModule::StartupModule()
{
	FBufferVisualizationMenuCommands::Register();
	FNaniteVisualizationMenuCommands::Register();
	FRayTracingDebugVisualizationMenuCommands::Register();
	FLumenVisualizationMenuCommands::Register();
	FMegaLightsVisualizationMenuCommands::Register();
	FSubstrateVisualizationMenuCommands::Register();
	FGroomVisualizationMenuCommands::Register();
	FVirtualShadowMapVisualizationMenuCommands::Register();
	FVirtualTextureVisualizationMenuCommands::Register();
	FShowFlagMenuCommands::Register();
}

void FCommonMenuExtensionsModule::ShutdownModule()
{
	FShowFlagMenuCommands::Unregister();
	FVirtualTextureVisualizationMenuCommands::Unregister();
	FVirtualShadowMapVisualizationMenuCommands::Unregister();
	FNaniteVisualizationMenuCommands::Unregister();
	FGroomVisualizationMenuCommands::Unregister();
	FSubstrateVisualizationMenuCommands::Unregister();
	FLumenVisualizationMenuCommands::Unregister();
	FMegaLightsVisualizationMenuCommands::Unregister();
	FRayTracingDebugVisualizationMenuCommands::Unregister();
	FBufferVisualizationMenuCommands::Unregister();
}
