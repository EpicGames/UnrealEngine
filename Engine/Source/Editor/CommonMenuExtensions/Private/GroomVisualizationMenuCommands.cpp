// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomVisualizationMenuCommands.h"

#include "Delegates/Delegate.h"
#include "EditorViewportClient.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "ToolMenus.h"
#include "GroomVisualizationData.h"
#include "Misc/AssertionMacros.h"
#include "Styling/AppStyle.h"
#include "UObject/UnrealNames.h"
#include "GroomVisualizationData.h"
#include "SEditorViewport.h"

#define LOCTEXT_NAMESPACE "GroomVisualizationMenuCommands"

UE_DEFINE_TCOMMANDS(FGroomVisualizationMenuCommands)

FGroomVisualizationMenuCommands::FGroomVisualizationMenuCommands()
	: TCommands<FGroomVisualizationMenuCommands>
	(
		TEXT("GroomVisualizationMenu"), // Context name for fast lookup
		NSLOCTEXT("Contexts", "GroomVisualizationMenu", "Groom"), // Localized context name for displaying
		NAME_None, // Parent context name.  
		FAppStyle::GetAppStyleSetName() // Icon Style Set
	),
	CommandMap()
{
}

void FGroomVisualizationMenuCommands::BuildCommandMap()
{
	const FGroomVisualizationData& VisualizationData = GetGroomVisualizationData();
	const FGroomVisualizationData::TModeMap& ModeMap = VisualizationData.GetModeMap();

	CommandMap.Empty();
	for (FGroomVisualizationData::TModeMap::TConstIterator It = ModeMap.CreateConstIterator(); It; ++It)
	{
		const FGroomVisualizationData::FModeRecord& Entry = It.Value();
		FGroomVisualizationRecord& Record = CommandMap.Add(Entry.ModeName, FGroomVisualizationRecord());
		Record.Name = Entry.ModeName;
		Record.Mode = Entry.Mode;
		Record.Command = FUICommandInfoDecl(
			this->AsShared(),
			Entry.ModeName,
			Entry.ModeText,
			Entry.ModeDesc)
			.UserInterfaceType(EUserInterfaceActionType::RadioButton)
			.DefaultChord(FInputChord()
		);

	}
}

void FGroomVisualizationMenuCommands::InternalBuildVisualisationSubMenu(FMenuBuilder& Menu, bool bIsGroomEditor)
{
	const FGroomVisualizationMenuCommands& Commands = FGroomVisualizationMenuCommands::Get();
	if (Commands.IsPopulated())
	{
		// General
		{
			Menu.BeginSection("LevelViewportGroomVisualizationGeneral", LOCTEXT("GroomVisualizationGeneral", "Groom General"));
			if (!bIsGroomEditor)
			{
				Commands.AddCommandTypeToMenu(Menu, EGroomViewMode::MacroGroups);
			}
			Commands.AddCommandTypeToMenu(Menu, EGroomViewMode::Group);
			Commands.AddCommandTypeToMenu(Menu, EGroomViewMode::LODColoration);
			Commands.AddCommandTypeToMenu(Menu, EGroomViewMode::Bounds);
			Menu.EndSection();
		}

		// Attributes
		{
			Menu.BeginSection("LevelViewportGroomVisualizationAttributes", LOCTEXT("GroomVisualizationAttributes", "Groom Attributes"));
			Commands.AddCommandTypeToMenu(Menu, EGroomViewMode::Seed);
			Commands.AddCommandTypeToMenu(Menu, EGroomViewMode::RootUV);
			Commands.AddCommandTypeToMenu(Menu, EGroomViewMode::RootUDIM);
			Commands.AddCommandTypeToMenu(Menu, EGroomViewMode::UV);
			Commands.AddCommandTypeToMenu(Menu, EGroomViewMode::Dimension);
			Commands.AddCommandTypeToMenu(Menu, EGroomViewMode::RadiusVariation);
			Commands.AddCommandTypeToMenu(Menu, EGroomViewMode::Tangent);
			Commands.AddCommandTypeToMenu(Menu, EGroomViewMode::ClumpID);
			Menu.EndSection();

		}
		// Guides
		{
			Menu.BeginSection("LevelViewportGroomVisualizationGuides", LOCTEXT("GroomVisualizationGuides", "Groom Guides"));
			Commands.AddCommandTypeToMenu(Menu, EGroomViewMode::SimHairStrands);
			Commands.AddCommandTypeToMenu(Menu, EGroomViewMode::RenderHairStrands);
			Commands.AddCommandTypeToMenu(Menu, EGroomViewMode::CardGuides);
			Menu.EndSection();
		}

		// Strands
		{
			Menu.BeginSection("LevelViewportGroomVisualizationStrands", LOCTEXT("GroomVisualizationMaterial", "Groom Strands"));
			Commands.AddCommandTypeToMenu(Menu, EGroomViewMode::ControlPoints);
			Commands.AddCommandTypeToMenu(Menu, EGroomViewMode::Color);
			Commands.AddCommandTypeToMenu(Menu, EGroomViewMode::Roughness);
			Commands.AddCommandTypeToMenu(Menu, EGroomViewMode::AO);
			Commands.AddCommandTypeToMenu(Menu, EGroomViewMode::MaterialDepth);
			Commands.AddCommandTypeToMenu(Menu, EGroomViewMode::MaterialBaseColor);
			Commands.AddCommandTypeToMenu(Menu, EGroomViewMode::MaterialRoughness);
			Commands.AddCommandTypeToMenu(Menu, EGroomViewMode::MaterialSpecular);
			Commands.AddCommandTypeToMenu(Menu, EGroomViewMode::MaterialTangent);
			Menu.EndSection();
		}

		// Advanced
		{
			Menu.BeginSection("LevelViewportGroomVisualizationAdvanced", LOCTEXT("GroomVisualizationAdvanced", "Groom Strands Advanced"));
			Commands.AddCommandTypeToMenu(Menu, EGroomViewMode::VoxelsDensity);
			//Commands.AddCommandTypeToMenu(Menu, EGroomViewMode::Cluster);
			Commands.AddCommandTypeToMenu(Menu, EGroomViewMode::ClusterAABB);
			if (!bIsGroomEditor)
			{
				Commands.AddCommandTypeToMenu(Menu, EGroomViewMode::LightBounds);
				Commands.AddCommandTypeToMenu(Menu, EGroomViewMode::DeepOpacityMaps);
				Commands.AddCommandTypeToMenu(Menu, EGroomViewMode::MacroGroupScreenRect);
				Commands.AddCommandTypeToMenu(Menu, EGroomViewMode::SamplePerPixel);
				Commands.AddCommandTypeToMenu(Menu, EGroomViewMode::CoverageType);
				Commands.AddCommandTypeToMenu(Menu, EGroomViewMode::TAAResolveType);
				Commands.AddCommandTypeToMenu(Menu, EGroomViewMode::MeshProjection); // Rename RootBinding
				Commands.AddCommandTypeToMenu(Menu, EGroomViewMode::Coverage);
				Commands.AddCommandTypeToMenu(Menu, EGroomViewMode::Tile);
			}
			Commands.AddCommandTypeToMenu(Menu, EGroomViewMode::Memory);
			Menu.EndSection();
		}
	}
}

void FGroomVisualizationMenuCommands::BuildVisualisationSubMenu(FMenuBuilder& Menu)
{
	InternalBuildVisualisationSubMenu(Menu, false);
}

void FGroomVisualizationMenuCommands::BuildVisualisationSubMenuForGroomEditor(FMenuBuilder& Menu)
{
	InternalBuildVisualisationSubMenu(Menu, true);
}

FToolMenuEntry FGroomVisualizationMenuCommands::BuildVisualizationSubMenuItem(const TWeakPtr<SEditorViewport>& Viewport)
{
	return InternalBuildVisualizationSubMenuItem(Viewport, false);
}

FToolMenuEntry FGroomVisualizationMenuCommands::BuildVisualizationSubMenuItemForGroomEditor(const TWeakPtr<SEditorViewport>& Viewport)
{
	return InternalBuildVisualizationSubMenuItem(Viewport, true);
}

FToolMenuEntry FGroomVisualizationMenuCommands::InternalBuildVisualizationSubMenuItem(const TWeakPtr<SEditorViewport>& Viewport, bool bIsGroomEditor)
{
	FToolMenuEntry MenuEntry = FToolMenuEntry::InitSubMenu(
		"VisualizeGroomViewMode",
		LOCTEXT("VisualizeGroomViewModeDisplayName", "Groom"),
		LOCTEXT("GroomVisualizationMenu_ToolTip", "Select a mode for Groom visualization"),
		FNewMenuDelegate::CreateStatic(
			bIsGroomEditor
			? &FGroomVisualizationMenuCommands::BuildVisualisationSubMenuForGroomEditor
			: &FGroomVisualizationMenuCommands::BuildVisualisationSubMenu
		),
		FUIAction(
			FExecuteAction(),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda(
				[WeakViewport = Viewport]()
				{
					if (TSharedPtr<SEditorViewport> Viewport = WeakViewport.Pin())
					{
						if (const TSharedPtr<FEditorViewportClient>& Client = Viewport->GetViewportClient())
						{
							return Client->IsViewModeEnabled(VMI_VisualizeGroom);
						}
					}
					return false;
				}
			)
		),
		EUserInterfaceActionType::RadioButton,
		/* bInOpenSubMenuOnClick = */ false,
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.VisualizeGroomMode")
	);
	
	return MenuEntry;
}

bool FGroomVisualizationMenuCommands::AddCommandTypeToMenu(FMenuBuilder& Menu, const EGroomViewMode Mode) const
{
	bool bAddedCommands = false;

	const TGroomVisualizationModeCommandMap& Commands = CommandMap;
	for (TCommandConstIterator It = CreateCommandConstIterator(); It; ++It)
	{
		const FGroomVisualizationRecord& Record = It.Value();
		if (Record.Mode == Mode)
		{
			Menu.AddMenuEntry(Record.Command, NAME_None, Record.Command->GetLabel());
			bAddedCommands = true;
		}
	}

	return bAddedCommands;
}

FGroomVisualizationMenuCommands::TCommandConstIterator FGroomVisualizationMenuCommands::CreateCommandConstIterator() const
{
	return CommandMap.CreateConstIterator();
}

void FGroomVisualizationMenuCommands::RegisterCommands()
{
	BuildCommandMap();
}

void FGroomVisualizationMenuCommands::BindCommands(FUICommandList& CommandList, const TSharedPtr<FEditorViewportClient>& Client) const
{
	// Map Groom visualization mode actions
	for (FGroomVisualizationMenuCommands::TCommandConstIterator It = FGroomVisualizationMenuCommands::Get().CreateCommandConstIterator(); It; ++It)
	{
		const FGroomVisualizationMenuCommands::FGroomVisualizationRecord& Record = It.Value();
		CommandList.MapAction(
			Record.Command,
			FExecuteAction::CreateStatic(&FGroomVisualizationMenuCommands::ChangeGroomVisualizationMode, Client.ToWeakPtr(), Record.Name),
			FCanExecuteAction(),
			FIsActionChecked::CreateStatic(&FGroomVisualizationMenuCommands::IsGroomVisualizationModeSelected, Client.ToWeakPtr(), Record.Name)
		);
	}
}

void FGroomVisualizationMenuCommands::ChangeGroomVisualizationMode(TWeakPtr<FEditorViewportClient> WeakClient, FName InName)
{
	if (TSharedPtr<FEditorViewportClient> Client = WeakClient.Pin())
	{
		Client->ChangeGroomVisualizationMode(InName);
	}
}

bool FGroomVisualizationMenuCommands::IsGroomVisualizationModeSelected(TWeakPtr<FEditorViewportClient> WeakClient, FName InName)
{
	if (TSharedPtr<FEditorViewportClient> Client = WeakClient.Pin())
	{
		return Client->IsGroomVisualizationModeSelected(InName);
	}
	
	return false;
}

#undef LOCTEXT_NAMESPACE
