// Copyright Epic Games, Inc. All Rights Reserved.

#include "RayTracingDebugVisualizationMenuCommands.h"

#include "DataDrivenShaderPlatformInfo.h"
#include "Containers/UnrealString.h"
#include "Framework/Commands/InputChord.h"
#include "Internationalization/Text.h"
#include "Templates/Function.h"
#include "RayTracingVisualizationData.h"
#include "Styling/AppStyle.h"
#include "EditorViewportClient.h"
#include "ToolMenu.h"

int32 GRayTracingVisualizeTiming = 0;
static FAutoConsoleVariableRef CVarRayTracingVisualizeTiming(
	TEXT("r.RayTracing.Visualize.Timing"),
	GRayTracingVisualizeTiming,
	TEXT("Include 'Timing' visualization modes in the in-editor 'Ray Tracing Debug' drop down menu.")
);

int32 GRayTracingVisualizeOther = 0;
static FAutoConsoleVariableRef CVarRayTracingVisualizeOther(
	TEXT("r.RayTracing.Visualize.Other"),
	GRayTracingVisualizeOther,
	TEXT("Include 'Other' visualization modes in the in-editor 'Ray Tracing Debug' drop down menu.")
);

#define LOCTEXT_NAMESPACE "RayTracingDebugVisualizationMenuCommands"

UE_DEFINE_TCOMMANDS(FRayTracingDebugVisualizationMenuCommands)

FRayTracingDebugVisualizationMenuCommands::FRayTracingDebugVisualizationMenuCommands()
	: TCommands<FRayTracingDebugVisualizationMenuCommands>
	(
		TEXT("RayTracingDebugVisualizationMenu"), // Context name for fast lookup
		NSLOCTEXT("Contexts", "RayTracingMenu", "Ray Tracing Debug Visualization"), // Localized context name for displaying
		NAME_None, // Parent context name.  
		FAppStyle::GetAppStyleSetName() // Icon Style Set
	),
	CommandMap()
{
}

void FRayTracingDebugVisualizationMenuCommands::BuildCommandMap()
{
	const FRayTracingVisualizationData& VisualizationData = GetRayTracingVisualizationData();
	const FRayTracingVisualizationData::TModeMap& ModeMap = VisualizationData.GetModeMap();

	CommandMap.Empty();
	for (FRayTracingVisualizationData::TModeMap::TConstIterator It = ModeMap.CreateConstIterator(); It; ++It)
	{
		const FRayTracingVisualizationData::FModeRecord& Entry = It.Value();

		if (Entry.bHiddenInEditor)
		{
			continue;
		}

		FVisualizationRecord& Record = CommandMap.Add(Entry.ModeName, FVisualizationRecord());
		Record.Name = Entry.ModeName;
		Record.Command = FUICommandInfoDecl(
			this->AsShared(),
			Entry.ModeName,
			Entry.ModeText,
			Entry.ModeDesc)
			.UserInterfaceType(EUserInterfaceActionType::RadioButton)
			.DefaultChord(FInputChord()
			);

		switch (Entry.ModeType)
		{
		case FRayTracingVisualizationData::FModeType::Overview:
			Record.Type = FVisualizationType::Overview;
			break;

		default:
		case FRayTracingVisualizationData::FModeType::Standard:
			Record.Type = FVisualizationType::Standard;
			break;

		case FRayTracingVisualizationData::FModeType::Performance:
			Record.Type = FVisualizationType::Performance;
			break;

		case FRayTracingVisualizationData::FModeType::Timing:
			Record.Type = FVisualizationType::Timing;
			break;

		case FRayTracingVisualizationData::FModeType::Other:
			Record.Type = FVisualizationType::Other;
			break;
		}
	}

	UI_COMMAND(ShowNearFieldCommand, "Near Field", "Visualizations will show the near field layer of the ray tracing scene (disables r.RayTracing.Visualize.FarField).", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ShowFarFieldCommand, "Far Field", "Visualizations will show the far field layer of the ray tracing scene (enables r.RayTracing.Visualize.FarField).", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND(ShowOpaqueOnlyCommand, "Opaque Only", "Whether to only show opaque instances in ray tracing debug visualizations (r.RayTracing.Visualize.OpaqueOnly).", EUserInterfaceActionType::ToggleButton, FInputChord());
}

void FRayTracingDebugVisualizationMenuCommands::BuildVisualisationSubMenu(UToolMenu* InMenu)
{
	if (!InMenu)
	{
		return;
	}

	const FRayTracingDebugVisualizationMenuCommands& Commands = FRayTracingDebugVisualizationMenuCommands::Get();
	if (!Commands.IsPopulated())
	{
		return;
	}

	{
		FToolMenuSection& UnnamedSection = InMenu->AddSection(NAME_None);
		Commands.AddCommandTypeToSection(UnnamedSection, FVisualizationType::Overview);
	}

	{
		FToolMenuSection& GeneralSection = InMenu->AddSection(
			"LevelViewportRayTracingVisualizationGeneral", LOCTEXT("RayTracingVisualizationGeneral", "General")
		);
		Commands.AddCommandTypeToSection(GeneralSection, FVisualizationType::Standard);
	}

	{
		FToolMenuSection& PerformanceSection = InMenu->AddSection(
			"LevelViewportRayTracingVisualizationPerformance",
			LOCTEXT("RayTracingVisualizationPerformance", "Performance")
		);
		Commands.AddCommandTypeToSection(PerformanceSection, FVisualizationType::Performance);
	}

	const bool bShowTiming = GRayTracingVisualizeTiming != 0;
	if (bShowTiming)
	{
		FToolMenuSection& TimingSection = InMenu->AddSection("LevelViewportRayTracingVisualizationTiming", LOCTEXT("RayTracingVisualizationTiming", "Timing"));
		Commands.AddCommandTypeToSection(TimingSection, FVisualizationType::Timing);
	}

	const bool bShowOther = GRayTracingVisualizeOther != 0;
	if (bShowOther)
	{
		FToolMenuSection& OtherSection = InMenu->AddSection("LevelViewportRayTracingVisualizationOther", LOCTEXT("RayTracingVisualizationOther", "Other"));
		Commands.AddCommandTypeToSection(OtherSection, FVisualizationType::Other);
	}

	Commands.AddLayersSection(*InMenu);
	Commands.AddOptionsSection(*InMenu);
}

bool FRayTracingDebugVisualizationMenuCommands::AddCommandTypeToSection(FToolMenuSection& InSection, const FVisualizationType Type) const
{
	bool bAddedCommands = false;

	for (TCommandConstIterator It = CreateCommandConstIterator(); It; ++It)
	{
		const FVisualizationRecord& Record = It.Value();
		if (Record.Type == Type)
		{
			InSection.AddMenuEntry(Record.Command, Record.Command->GetLabel());
			bAddedCommands = true;
		}
	}

	return bAddedCommands;
}

bool FRayTracingDebugVisualizationMenuCommands::AddLayersSection(UToolMenu& InMenu) const
{
	FToolMenuSection& Section = InMenu.AddSection("LevelViewportRayTracingVisualizationLayers", LOCTEXT("RayTracingVisualizationLayers", "Layers"));
	Section.AddMenuEntry(ShowNearFieldCommand);
	Section.AddMenuEntry(ShowFarFieldCommand);

	return true;
}

bool FRayTracingDebugVisualizationMenuCommands::AddOptionsSection(UToolMenu& InMenu) const
{
	FToolMenuSection& Section = InMenu.AddSection("LevelViewportRayTracingVisualizationOptions", LOCTEXT("RayTracingVisualizationOptions", "Options"));
	Section.AddMenuEntry(ShowOpaqueOnlyCommand);

	return true;
}

FRayTracingDebugVisualizationMenuCommands::TCommandConstIterator FRayTracingDebugVisualizationMenuCommands::CreateCommandConstIterator() const
{
	return CommandMap.CreateConstIterator();
}

void FRayTracingDebugVisualizationMenuCommands::RegisterCommands()
{
	BuildCommandMap();
}

void FRayTracingDebugVisualizationMenuCommands::BindCommands(FUICommandList& CommandList, const TSharedPtr<FEditorViewportClient>& Client) const
{
	// Map ray tracing debug visualization mode actions
	for (FRayTracingDebugVisualizationMenuCommands::TCommandConstIterator It = FRayTracingDebugVisualizationMenuCommands::Get().CreateCommandConstIterator(); It; ++It)
	{
		const FRayTracingDebugVisualizationMenuCommands::FVisualizationRecord& Record = It.Value();
		CommandList.MapAction(
			Record.Command,
			FExecuteAction::CreateStatic(&FRayTracingDebugVisualizationMenuCommands::ChangeRayTracingDebugVisualizationMode, Client.ToWeakPtr(), Record.Name),
			FCanExecuteAction(),
			FIsActionChecked::CreateStatic(&FRayTracingDebugVisualizationMenuCommands::IsRayTracingDebugVisualizationModeSelected, Client.ToWeakPtr(), Record.Name)
		);
	}

	TWeakPtr<FEditorViewportClient> WeakClient = Client;
	IConsoleVariable* FarFieldCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.RayTracing.Visualize.FarField"));
	check(FarFieldCVar);

	CommandList.MapAction(
		ShowNearFieldCommand,
		FExecuteAction::CreateLambda([=]()
			{
				FarFieldCVar->Set(false, ECVF_SetByConsole);

				if (TSharedPtr<FEditorViewportClient> Client = WeakClient.Pin())
				{
					Client->Invalidate();
				}
			}),
		FCanExecuteAction::CreateLambda([]() { return true; }),
		FIsActionChecked::CreateLambda([=]() { return !FarFieldCVar->GetBool(); })
	);

	CommandList.MapAction(
		ShowFarFieldCommand,
		FExecuteAction::CreateLambda([=]()
			{
				FarFieldCVar->Set(true, ECVF_SetByConsole);
				
				if (TSharedPtr<FEditorViewportClient> Client = WeakClient.Pin())
				{
					Client->Invalidate();
				}
			}),
		FCanExecuteAction::CreateLambda([]() { return true; }),
		FIsActionChecked::CreateLambda([=]() { return FarFieldCVar->GetBool(); })
	);

	IConsoleVariable* OpaqueOnlyCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.RayTracing.Visualize.OpaqueOnly"));
	check(OpaqueOnlyCVar);

	CommandList.MapAction(
		ShowOpaqueOnlyCommand,
		FExecuteAction::CreateLambda([=]()
			{
				OpaqueOnlyCVar->Set(!OpaqueOnlyCVar->GetBool());

				if (TSharedPtr<FEditorViewportClient> Client = WeakClient.Pin())
				{
					Client->Invalidate();
				}
			}),
		FCanExecuteAction::CreateLambda([]() { return true; }),
		FIsActionChecked::CreateLambda([=]() { return OpaqueOnlyCVar->GetBool(); })
	);
}

void FRayTracingDebugVisualizationMenuCommands::ChangeRayTracingDebugVisualizationMode(TWeakPtr<FEditorViewportClient> WeakClient, FName InName)
{
	if (TSharedPtr<FEditorViewportClient> Client = WeakClient.Pin())
	{
		Client->ChangeRayTracingDebugVisualizationMode(InName);
	}
}

bool FRayTracingDebugVisualizationMenuCommands::IsRayTracingDebugVisualizationModeSelected(TWeakPtr<FEditorViewportClient> WeakClient, FName InName)
{
	if (TSharedPtr<FEditorViewportClient> Client = WeakClient.Pin())
	{
		return Client->IsRayTracingDebugVisualizationModeSelected(InName);
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
