// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShowFlagFilter.h"

#include "EditorShowFlags.h"
#include "HAL/PlatformCrt.h"

namespace
{
	inline bool FlagGroupExistsInArray(const TArray<EShowFlagGroup>& Array, const FShowFlagData& Flag)
	{
		return Array.FindByPredicate([&Flag](EShowFlagGroup Group)
		{
			return Flag.Group == Group;
		}) != NULL;
	}
}

FShowFlagFilter::FShowFlagFilter(EDefaultMode InDefaultMode)
	: DefaultMode(InDefaultMode),
	  IncludedGroups(),
	  ExcludedGroups(),
	  IncludedFlags(),
	  ExcludedFlags(),
	  AllowedFlagIndices(),
	  bHaveFiltered(false)
{
}

const FShowFlagFilter::FGroupedShowFlagIndices& FShowFlagFilter::GetFilteredIndices(bool bForceRebuild) const
{
	if ( bForceRebuild || !bHaveFiltered )
	{
		RebuildIndexArray();
		bHaveFiltered = true;
	}

	return AllowedFlagIndices;
}

void FShowFlagFilter::RebuildIndexArray() const
{
	AllowedFlagIndices.ClearAll();

	const TArray<FShowFlagData>& AllShowFlags = GetShowFlagMenuItems();

	for (uint32 Index = 0; Index < static_cast<uint32>(AllShowFlags.Num()); ++Index)
	{
		const FShowFlagData& Flag = AllShowFlags[Index];
		if (!ShouldIncludeFlag(Flag))
		{
			continue;
		}

		AllowedFlagIndices[Flag.Group].Add(Index);
	}
}

bool FShowFlagFilter::ShouldIncludeFlag(const FShowFlagData& Flag) const
{
	if ( IncludedFlags.Contains(static_cast<FEngineShowFlags::EShowFlag>(Flag.EngineShowFlagIndex)) )
	{
		return true;
	}

	if ( ExcludedFlags.Contains(static_cast<FEngineShowFlags::EShowFlag>(Flag.EngineShowFlagIndex)) )
	{
		return false;
	}

	if ( FlagGroupExistsInArray(IncludedGroups, Flag) )
	{
		return true;
	}

	if ( FlagGroupExistsInArray(ExcludedGroups, Flag) )
	{
		return false;
	}

	return DefaultMode == IncludeAllFlagsByDefault;
}

FShowFlagFilter& FShowFlagFilter::IncludeFlag(FEngineShowFlags::EShowFlag Flag)
{
	IncludedFlags.Add(Flag);
	return *this;
}

FShowFlagFilter& FShowFlagFilter::ExcludeFlag(FEngineShowFlags::EShowFlag Flag)
{
	ExcludedFlags.Add(Flag);
	return *this;
}

FShowFlagFilter& FShowFlagFilter::IncludeGroup(EShowFlagGroup Group)
{
	IncludedGroups.Add(Group);
	return *this;
}

FShowFlagFilter& FShowFlagFilter::ExcludeGroup(EShowFlagGroup Group)
{
	ExcludedGroups.Add(Group);
	return *this;
}