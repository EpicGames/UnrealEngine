// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShowFlagMenuCommands.h"

#include "Containers/BitArray.h"
#include "Containers/StringConv.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EditorShowFlags.h"
#include "EditorViewportClient.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"
#include "Internationalization/Internationalization.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Templates/Function.h"
#include "ToolMenu.h"
#include "ToolMenuDelegates.h"
#include "ToolMenuSection.h"
#include "UObject/UnrealNames.h"

#define LOCTEXT_NAMESPACE "ShowFlagMenuCommands"

namespace
{
	inline FText GetLocalizedShowFlagName(const FShowFlagData& Flag)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("ShowFlagName"), Flag.DisplayName);

		switch (Flag.Group)
		{
			case SFG_Visualize:
			{
				return FText::Format(LOCTEXT("VisualizeFlagLabel", "Visualize {ShowFlagName}"), Args);
			}
			
			default:
			{
				return FText::Format(LOCTEXT("ShowFlagLabel", "Show {ShowFlagName}"), Args);
			}
		}
	}
}

UE_DEFINE_TCOMMANDS(FShowFlagMenuCommands)

FShowFlagMenuCommands::FShowFlagMenuCommands()
	: TCommands<FShowFlagMenuCommands>
	(
		TEXT("ShowFlagsMenu"), // Context name for fast lookup
		NSLOCTEXT("Contexts", "ShowFlagsMenu", "Show Flags Menu"), // Localized context name for displaying
		NAME_None, // Parent context name.  
		FAppStyle::GetAppStyleSetName() // Icon Style Set
	),
	ShowFlagCommands(),
	bCommandsInitialised(false)
{
}

void FShowFlagMenuCommands::UpdateCustomShowFlags() const
{
	if (GetShowFlagMenuItems().Num() != ShowFlagCommands.Num())
	{
		const_cast<FShowFlagMenuCommands*>(this)->CreateShowFlagCommands();
	}
}

void FShowFlagMenuCommands::BuildShowFlagsMenu(UToolMenu* Menu, const FShowFlagFilter& Filter) const
{
	check(bCommandsInitialised);

	UpdateCustomShowFlags();

	const FShowFlagFilter::FGroupedShowFlagIndices& FlagIndices = Filter.GetFilteredIndices();
	if ( FlagIndices.TotalIndices() < 1 )
	{
		return;
	}

	{
		FToolMenuSection& Section =
			Menu->AddSection("ShowFlagsMenuSectionCommon", LOCTEXT("CommonShowFlagHeader", "Common Show Flags"));

		PopulateCommonShowFlagsSection(Section, Filter);
	}

	{
		FToolMenuSection& Section = Menu->AddSection("LevelViewportShowFlags", LOCTEXT("AllShowFlagHeader", "All Show Flags"));

		PopulateAllShowFlagsSection(Section, Filter);
	}
}

void FShowFlagMenuCommands::PopulateCommonShowFlagsSection(FToolMenuSection& Section, const FShowFlagFilter& Filter) const
{
	const FShowFlagFilter::FGroupedShowFlagIndices& GroupedFlagIndices = Filter.GetFilteredIndices();
	const TArray<uint32>& FlagIndices = GroupedFlagIndices[SFG_Normal];

	if (FlagIndices.Num() < 1)
	{
		return;
	}

	for (int32 ArrayIndex = 0; ArrayIndex < FlagIndices.Num(); ++ArrayIndex)
	{
		const uint32 FlagIndex = FlagIndices[ArrayIndex];
		const FShowFlagCommand& ShowFlagCommand = ShowFlagCommands[FlagIndex];

		ensure(Section.FindEntry(ShowFlagCommand.ShowMenuItem->GetCommandName()) == nullptr);
		Section.AddMenuEntry(
			*FString::Printf(TEXT("Common_%s"), *ShowFlagCommand.ShowMenuItem->GetCommandName().ToString()),
			ShowFlagCommand.ShowMenuItem,
			ShowFlagCommand.LabelOverride
		);
	}
}

void FShowFlagMenuCommands::PopulateAllShowFlagsSection(FToolMenuSection& Section, const FShowFlagFilter& Filter) const
{
	CreateSubMenuIfRequired(
		Section,
		Filter,
		SFG_PostProcess,
		"SFG_PostProcess",
		LOCTEXT("PostProcessShowFlagsMenu", "Post Processing"),
		LOCTEXT("PostProcessShowFlagsMenu_ToolTip", "Post process show flags"),
		"ShowFlagsMenu.SubMenu.PostProcessing"
	);
	CreateSubMenuIfRequired(
		Section,
		Filter,
		SFG_LightTypes,
		"SFG_LightTypes",
		LOCTEXT("LightTypesShowFlagsMenu", "Light Types"),
		LOCTEXT("LightTypesShowFlagsMenu_ToolTip", "Light Types show flags"),
		"ShowFlagsMenu.SubMenu.LightTypes"
	);
	CreateSubMenuIfRequired(
		Section,
		Filter,
		SFG_LightingComponents,
		"SFG_LightingComponents",
		LOCTEXT("LightingComponentsShowFlagsMenu", "Lighting Components"),
		LOCTEXT("LightingComponentsShowFlagsMenu_ToolTip", "Lighting Components show flags"),
		"ShowFlagsMenu.SubMenu.LightingComponents"
	);
	CreateSubMenuIfRequired(
		Section,
		Filter,
		SFG_LightingFeatures,
		"SFG_LightingFeatures",
		LOCTEXT("LightingFeaturesShowFlagsMenu", "Lighting Features"),
		LOCTEXT("LightingFeaturesShowFlagsMenu_ToolTip", "Lighting Features show flags"),
		"ShowFlagsMenu.SubMenu.LightingFeatures"
	);
	CreateSubMenuIfRequired(
		Section,
		Filter,
		SFG_Lumen,
		"SFG_Lumen",
		LOCTEXT("LumenShowFlagsMenu", "Lumen"),
		LOCTEXT("LumenShowFlagsMenu_ToolTip", "Lumen show flags"),
		"ShowFlagsMenu.SubMenu.Lumen"
	);
	CreateSubMenuIfRequired(
		Section,
		Filter,
		SFG_MegaLights,
		"SFG_MegaLights",
		LOCTEXT("MegaLightsShowFlagsMenu", "MegaLights"),
		LOCTEXT("MegaLightsShowFlagsMenu_ToolTip", "MegaLights show flags"),
		"ShowFlagsMenu.SubMenu.MegaLights"
	);
	CreateSubMenuIfRequired(
		Section,
		Filter,
		SFG_Nanite,
		"SFG_Nanite",
		LOCTEXT("NaniteShowFlagsMenu", "Nanite"),
		LOCTEXT("NaniteShowFlagsMenu_ToolTip", "Nanite show flags"),
		"ShowFlagsMenu.SubMenu.Nanite"
	);
	CreateSubMenuIfRequired(
		Section,
		Filter,
		SFG_Developer,
		"SFG_Developer",
		LOCTEXT("DeveloperShowFlagsMenu", "Developer"),
		LOCTEXT("DeveloperShowFlagsMenu_ToolTip", "Developer show flags"),
		"ShowFlagsMenu.SubMenu.Developer"
	);
	CreateSubMenuIfRequired(
		Section,
		Filter,
		SFG_Visualize,
		"SFG_Visualize",
		LOCTEXT("VisualizeShowFlagsMenu", "Visualize"),
		LOCTEXT("VisualizeShowFlagsMenu_ToolTip", "Visualize show flags"),
		"ShowFlagsMenu.SubMenu.Visualize"
	);
	CreateSubMenuIfRequired(
		Section,
		Filter,
		SFG_Advanced,
		"SFG_Advanced",
		LOCTEXT("AdvancedShowFlagsMenu", "Advanced"),
		LOCTEXT("AdvancedShowFlagsMenu_ToolTip", "Advanced show flags"),
		"ShowFlagsMenu.SubMenu.Advanced"
	);
	CreateSubMenuIfRequired(
		Section,
		Filter,
		SFG_Custom,
		"SFG_Custom",
		LOCTEXT("CustomShowFlagsMenu", "Custom"),
		LOCTEXT("CustomShowFlagsMenu_ToolTip", "Custom show flags"),
		NAME_None
	);
}

const TArray<FShowFlagMenuCommands::FShowFlagCommand>& FShowFlagMenuCommands::GetCommands() const
{
	return ShowFlagCommands;
}

TSharedPtr<const FUICommandInfo> FShowFlagMenuCommands::FindCommandForFlag(FEngineShowFlags::EShowFlag FlagIndex) const
{
	for (int32 Index = 0; Index < ShowFlagCommands.Num(); ++Index)
	{
		if (ShowFlagCommands[Index].FlagIndex == FlagIndex)
		{
			return ShowFlagCommands[Index].ShowMenuItem;
		}
	}
	return nullptr;
}

void FShowFlagMenuCommands::CreateSubMenuIfRequired(FToolMenuSection& Section, const FShowFlagFilter& Filter, EShowFlagGroup Group, const FName SubMenuName, const FText& MenuLabel, const FText& ToolTip, const FName IconName) const
{
	const FShowFlagFilter::FGroupedShowFlagIndices& GroupedFlagIndices = Filter.GetFilteredIndices();
	const TArray<uint32>& FlagIndices = GroupedFlagIndices[Group];

	if ( FlagIndices.Num() < 1 )
	{
		return;
	}

	Section.AddSubMenu(SubMenuName, MenuLabel, ToolTip, FNewToolMenuDelegate::CreateStatic(&FShowFlagMenuCommands::StaticCreateShowFlagsSubMenu, FlagIndices, 0), false, FSlateIcon(FAppStyle::Get().GetStyleSetName(), IconName));
}

void FShowFlagMenuCommands::CreateShowFlagsSubMenu(UToolMenu* Menu, TArray<uint32> FlagIndices, int32 EntryOffset) const
{
	// Generate entries for the standard show flags.
	// Assumption: the first 'n' entries types like 'Show All' and 'Hide All' buttons, so insert a separator after them.

	FToolMenuSection& Section = Menu->AddSection("Section");
	for (int32 ArrayIndex = 0; ArrayIndex < FlagIndices.Num(); ++ArrayIndex)
	{
		const uint32 FlagIndex = FlagIndices[ArrayIndex];
		const FShowFlagCommand& ShowFlagCommand = ShowFlagCommands[FlagIndex];

		ensure(Section.FindEntry(ShowFlagCommand.ShowMenuItem->GetCommandName()) == nullptr);

		FFormatNamedArguments Args;
		Args.Add(TEXT("ShowFlagDefault"), FText::AsCultureInvariant("ShowFlag." + FEngineShowFlags::FindNameByIndex(ShowFlagCommand.FlagIndex) + " 2"));

		FText OverrideEnabledWarning = FEngineShowFlags::IsForceFlagSet(ShowFlagCommand.FlagIndex)
			? FText::GetEmpty()
			: FText::Format(LOCTEXT("ShowFlagOverrideWarning", "ShowFlag override on. Set to default in console to use Editor UI (Set: \"{ShowFlagDefault}\")."), Args);

		Section.AddMenuEntry(ShowFlagCommand.ShowMenuItem->GetCommandName(), ShowFlagCommand.ShowMenuItem, ShowFlagCommand.LabelOverride, OverrideEnabledWarning);

		if (ArrayIndex == EntryOffset - 1)
		{
			Section.AddSeparator(NAME_None);
		}
	}
}

void FShowFlagMenuCommands::RegisterCommands()
{
	CreateShowFlagCommands();
	bCommandsInitialised = true;
}

void FShowFlagMenuCommands::CreateShowFlagCommands()
{
	TBitArray<> ExistingCommands;

	int32 MaxIndex = 0;
	for (const FShowFlagCommand& Command : ShowFlagCommands)
	{
		MaxIndex = FMath::Max(MaxIndex, (int32)Command.FlagIndex);
	}

	ExistingCommands.Init(false, MaxIndex + 1);
	for (const FShowFlagCommand& Command : ShowFlagCommands)
	{
		ExistingCommands[Command.FlagIndex] = true;
	}

	const TArray<FShowFlagData>& AllShowFlags = GetShowFlagMenuItems();

	for (int32 ShowFlagIndex = 0; ShowFlagIndex < AllShowFlags.Num(); ++ShowFlagIndex)
	{
		const FShowFlagData& ShowFlag = AllShowFlags[ShowFlagIndex];
		if ((int32)ShowFlag.EngineShowFlagIndex < ExistingCommands.Num() && ExistingCommands[ShowFlag.EngineShowFlagIndex])
		{
			continue;
		}

		const FText LocalizedName = GetLocalizedShowFlagName(ShowFlag);

		//@todo Slate: The show flags system does not support descriptions currently
		const FText ShowFlagDesc;

		TSharedPtr<FUICommandInfo> ShowFlagCommand = FUICommandInfoDecl(this->AsShared(), ShowFlag.ShowFlagName, LocalizedName, ShowFlagDesc)
			.UserInterfaceType(EUserInterfaceActionType::ToggleButton)
			.DefaultChord(ShowFlag.InputChord)
			.Icon(GetShowFlagIcon(ShowFlag));

		ShowFlagCommands.Add(FShowFlagCommand(static_cast<FEngineShowFlags::EShowFlag>(ShowFlag.EngineShowFlagIndex), ShowFlagCommand, ShowFlag.DisplayName));
	}
}

void FShowFlagMenuCommands::StaticCreateShowFlagsSubMenu(UToolMenu* Menu, TArray<uint32> FlagIndices, int32 EntryOffset)
{
	FShowFlagMenuCommands::Get().CreateShowFlagsSubMenu(Menu, FlagIndices, EntryOffset);
}

FSlateIcon FShowFlagMenuCommands::GetShowFlagIcon(const FShowFlagData& Flag) const
{
	return Flag.Group == EShowFlagGroup::SFG_Normal
		? FSlateIcon(FAppStyle::GetAppStyleSetName(), FAppStyle::Join(GetContextName(), TCHAR_TO_ANSI(*FString::Printf(TEXT(".%s"), *Flag.ShowFlagName.ToString()))))
		: FSlateIcon();
}

void FShowFlagMenuCommands::BindCommands(FUICommandList& CommandList, const TSharedPtr<FEditorViewportClient>& Client) const
{
	check(bCommandsInitialised);
	check(Client.IsValid());

	UpdateCustomShowFlags();

	for (int32 ArrayIndex = 0; ArrayIndex < ShowFlagCommands.Num(); ++ArrayIndex)
	{
		const FShowFlagCommand& ShowFlagCommand = ShowFlagCommands[ArrayIndex];

		CommandList.MapAction(ShowFlagCommand.ShowMenuItem,
							  FExecuteAction::CreateStatic(&FShowFlagMenuCommands::ToggleShowFlag, Client.ToWeakPtr(), ShowFlagCommand.FlagIndex),
							  FCanExecuteAction::CreateStatic(&FEngineShowFlags::IsForceFlagSet, static_cast<uint32>(ShowFlagCommand.FlagIndex)),
							  FIsActionChecked::CreateStatic(&FShowFlagMenuCommands::IsShowFlagEnabled, Client.ToWeakPtr(), ShowFlagCommand.FlagIndex));
	}
}

void FShowFlagMenuCommands::ToggleShowFlag(TWeakPtr<FEditorViewportClient> WeakClient, FEngineShowFlags::EShowFlag EngineShowFlagIndex)
{
	if (TSharedPtr<FEditorViewportClient> Client = WeakClient.Pin())
	{
		Client->HandleToggleShowFlag(EngineShowFlagIndex);
	}
}

bool FShowFlagMenuCommands::IsShowFlagEnabled(TWeakPtr<FEditorViewportClient> WeakClient, FEngineShowFlags::EShowFlag EngineShowFlagIndex)
{
	if (TSharedPtr<FEditorViewportClient> Client = WeakClient.Pin())
	{
		return Client->HandleIsShowFlagEnabled(EngineShowFlagIndex);
	}

	return false;
}

#undef LOCTEXT_NAMESPACE