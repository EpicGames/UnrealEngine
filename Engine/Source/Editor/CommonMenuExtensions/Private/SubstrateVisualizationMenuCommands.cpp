// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubstrateVisualizationMenuCommands.h"

#include "Delegates/Delegate.h"
#include "EditorViewportClient.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "SubstrateVisualizationData.h"
#include "Misc/AssertionMacros.h"
#include "Styling/AppStyle.h"
#include "UObject/UnrealNames.h"

#define LOCTEXT_NAMESPACE "SubstrateVisualizationMenuCommands"

UE_DEFINE_TCOMMANDS(FSubstrateVisualizationMenuCommands)

FSubstrateVisualizationMenuCommands::FSubstrateVisualizationMenuCommands()
	: TCommands<FSubstrateVisualizationMenuCommands>
	(
		TEXT("SubstrateVisualizationMenu"), // Context name for fast lookup
		NSLOCTEXT("Contexts", "SubstrateVisualizationMenu", "Substrate"), // Localized context name for displaying
		NAME_None, // Parent context name.  
		FAppStyle::GetAppStyleSetName() // Icon Style Set
	),
	CommandMap()
{
}

void FSubstrateVisualizationMenuCommands::BuildCommandMap()
{
	const FSubstrateVisualizationData& VisualizationData = GetSubstrateVisualizationData();
	const FSubstrateVisualizationData::TModeMap& ModeMap = VisualizationData.GetModeMap();

	CommandMap.Empty();
	for (FSubstrateVisualizationData::TModeMap::TConstIterator It = ModeMap.CreateConstIterator(); It; ++It)
	{
		const FSubstrateVisualizationData::FModeRecord& Entry = It.Value();
		FSubstrateVisualizationRecord& Record = CommandMap.Add(Entry.ModeName, FSubstrateVisualizationRecord());
		Record.Name = Entry.ModeName;
		Record.Command = FUICommandInfoDecl(
			this->AsShared(),
			Entry.ModeName,
			Entry.ModeText,
			Entry.ModeDesc)
			.UserInterfaceType(EUserInterfaceActionType::RadioButton)
			.DefaultChord(FInputChord()
		);
		Record.ViewMode = Entry.ViewMode;
	}
}

void FSubstrateVisualizationMenuCommands::BuildVisualisationSubMenu(FMenuBuilder& Menu)
{
	const FSubstrateVisualizationMenuCommands& Commands = FSubstrateVisualizationMenuCommands::Get();
	if (Commands.IsPopulated())
	{
		// General
		{
			Menu.BeginSection("LevelViewportSubstrateVisualizationModeGeneral", LOCTEXT("SubstrateVisualizationGeneral", "Substrate General View Mode"));
			Commands.AddCommandTypeToMenu(Menu, FSubstrateViewMode::MaterialProperties);
			Commands.AddCommandTypeToMenu(Menu, FSubstrateViewMode::MaterialCount);
			Commands.AddCommandTypeToMenu(Menu, FSubstrateViewMode::MaterialByteCount);
			Commands.AddCommandTypeToMenu(Menu, FSubstrateViewMode::SubstrateInfo);
			Menu.EndSection();
		}

		// Advanced
		{
			Menu.BeginSection("LevelViewportSubstrateVisualizationModeAdvanced", LOCTEXT("SubstrateVisualizationAdvanced", "Substrate Advanced View Mode"));
			Commands.AddCommandTypeToMenu(Menu, FSubstrateViewMode::AdvancedMaterialProperties);
			Commands.AddCommandTypeToMenu(Menu, FSubstrateViewMode::MaterialClassification);
			Commands.AddCommandTypeToMenu(Menu, FSubstrateViewMode::RoughRefractionClassification);
			Commands.AddCommandTypeToMenu(Menu, FSubstrateViewMode::DecalClassification);
			Menu.EndSection();
		}
	}
}

bool FSubstrateVisualizationMenuCommands::AddCommandTypeToMenu(FMenuBuilder& Menu, const FSubstrateViewMode ViewMode) const
{
	bool bAddedCommands = false;

	const TSubstrateVisualizationModeCommandMap& Commands = CommandMap;
	for (TCommandConstIterator It = CreateCommandConstIterator(); It; ++It)
	{
		const FSubstrateVisualizationRecord& Record = It.Value();
		if (Record.ViewMode == ViewMode)
		{
			Menu.AddMenuEntry(Record.Command, NAME_None, Record.Command->GetLabel());
			bAddedCommands = true;
		}
	}

	return bAddedCommands;
}

FSubstrateVisualizationMenuCommands::TCommandConstIterator FSubstrateVisualizationMenuCommands::CreateCommandConstIterator() const
{
	return CommandMap.CreateConstIterator();
}

void FSubstrateVisualizationMenuCommands::RegisterCommands()
{
	BuildCommandMap();
}

void FSubstrateVisualizationMenuCommands::BindCommands(FUICommandList& CommandList, const TSharedPtr<FEditorViewportClient>& Client) const
{
	// Map Substrate visualization mode actions
	for (FSubstrateVisualizationMenuCommands::TCommandConstIterator It = FSubstrateVisualizationMenuCommands::Get().CreateCommandConstIterator(); It; ++It)
	{
		const FSubstrateVisualizationMenuCommands::FSubstrateVisualizationRecord& Record = It.Value();
		CommandList.MapAction(
			Record.Command,
			FExecuteAction::CreateStatic(&FSubstrateVisualizationMenuCommands::ChangeSubstrateVisualizationMode, Client.ToWeakPtr(), Record.Name),
			FCanExecuteAction(),
			FIsActionChecked::CreateStatic(&FSubstrateVisualizationMenuCommands::IsSubstrateVisualizationModeSelected, Client.ToWeakPtr(), Record.Name)
		);
	}
}

void FSubstrateVisualizationMenuCommands::ChangeSubstrateVisualizationMode(TWeakPtr<FEditorViewportClient> WeakClient, FName InName)
{
	if (TSharedPtr<FEditorViewportClient> Client = WeakClient.Pin())
	{
		Client->ChangeSubstrateVisualizationMode(InName);
	}
}

bool FSubstrateVisualizationMenuCommands::IsSubstrateVisualizationModeSelected(TWeakPtr<FEditorViewportClient> WeakClient, FName InName)
{
	if (TSharedPtr<FEditorViewportClient> Client = WeakClient.Pin())
	{
		return Client->IsSubstrateVisualizationModeSelected(InName);
	}
	
	return false;
}

#undef LOCTEXT_NAMESPACE
