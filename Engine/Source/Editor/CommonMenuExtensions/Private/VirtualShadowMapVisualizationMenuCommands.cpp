// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualShadowMapVisualizationMenuCommands.h"

#include "Delegates/Delegate.h"
#include "EditorViewportClient.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/IConsoleManager.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Misc/AssertionMacros.h"
#include "Styling/AppStyle.h"
#include "UObject/UnrealNames.h"
#include "VirtualShadowMapVisualizationData.h"

int32 GVirtualShadowMapVisualizeAdvanced = 0;
static FAutoConsoleVariableRef CVarVirtualShadowMapVisualizeAdvanced(
	TEXT("r.Shadow.Virtual.Visualize.Advanced"),
	GVirtualShadowMapVisualizeAdvanced,
	TEXT("Enable to show advanced VSM debug modes in the visualization UI menu.")
);

#define LOCTEXT_NAMESPACE "VirtualShadowMapVisualizationMenuCommands"

UE_DEFINE_TCOMMANDS(FVirtualShadowMapVisualizationMenuCommands)

FVirtualShadowMapVisualizationMenuCommands::FVirtualShadowMapVisualizationMenuCommands()
	: TCommands<FVirtualShadowMapVisualizationMenuCommands>
	(
		TEXT("VirtualShadowMapVisualizationMenu"), // Context name for fast lookup
		NSLOCTEXT("Contexts", "VirtualShadowMapVisualizationMenu", "VirtualShadowMap Visualization"), // Localized context name for displaying
		NAME_None, // Parent context name.  
		FAppStyle::GetAppStyleSetName() // Icon Style Set
	),
	CommandMap()
{
}

void FVirtualShadowMapVisualizationMenuCommands::BuildCommandMap()
{
	const FVirtualShadowMapVisualizationData& VisualizationData = GetVirtualShadowMapVisualizationData();
	const FVirtualShadowMapVisualizationData::TModeMap& ModeMap = VisualizationData.GetModeMap();

	CommandMap.Empty();
	for (FVirtualShadowMapVisualizationData::TModeMap::TConstIterator It = ModeMap.CreateConstIterator(); It; ++It)
	{
		const FVirtualShadowMapVisualizationData::FModeRecord& Entry = It.Value();
		FVirtualShadowMapVisualizationRecord& Record = CommandMap.Add(Entry.ModeName, FVirtualShadowMapVisualizationRecord());
		Record.Name = Entry.ModeName;
		Record.Command = FUICommandInfoDecl(
			this->AsShared(),
			Entry.ModeName,
			Entry.ModeText,
			Entry.ModeDesc)
			.UserInterfaceType(EUserInterfaceActionType::RadioButton)
			.DefaultChord(FInputChord());

		switch (Entry.ModeType)
		{
		case FVirtualShadowMapVisualizationData::FModeType::Standard:
			Record.Type = FVirtualShadowMapVisualizationType::Standard;
			break;

		default:
		case FVirtualShadowMapVisualizationData::FModeType::Advanced:
			Record.Type = FVirtualShadowMapVisualizationType::Advanced;
			break;
		}
	}

	UI_COMMAND(ShowStatsCommand, "Show Statistics", "r.Shadow.Virtual.ShowStats", EUserInterfaceActionType::Check, FInputChord());
	UI_COMMAND(VisualizeNextLightCommand, "Visualize next light", "r.Shadow.Virtual.Visualize.NextLight", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(VisualizePrevLightCommand, "Visualize previous light", "r.Shadow.Virtual.Visualize.PrevLight", EUserInterfaceActionType::Button, FInputChord());
}

void FVirtualShadowMapVisualizationMenuCommands::BuildVisualisationSubMenu(FMenuBuilder& Menu)
{
	const bool bShowAdvanced = GVirtualShadowMapVisualizeAdvanced != 0;

	const FVirtualShadowMapVisualizationMenuCommands& Commands = FVirtualShadowMapVisualizationMenuCommands::Get();
	if (Commands.IsPopulated())
	{
		Menu.BeginSection("LevelViewportVirtualShadowMapVisualizationMode", LOCTEXT("VirtualShadowMapVisualizationHeader", "Visualization Mode"));

		Commands.AddCommandTypeToMenu(Menu, FVirtualShadowMapVisualizationType::Standard, false);
		if (bShowAdvanced)
		{
			Commands.AddCommandTypeToMenu(Menu, FVirtualShadowMapVisualizationType::Advanced, true);
		}

		Menu.EndSection();
	}
}

bool FVirtualShadowMapVisualizationMenuCommands::AddCommandTypeToMenu(FMenuBuilder& Menu, const FVirtualShadowMapVisualizationType Type, bool bSeparatorBefore) const
{
	bool bAddedCommands = false;

	Menu.AddMenuEntry(ShowStatsCommand, NAME_None, ShowStatsCommand->GetLabel());

	const TVirtualShadowMapVisualizationModeCommandMap& Commands = CommandMap;
	for (TCommandConstIterator It = CreateCommandConstIterator(); It; ++It)
	{
		const FVirtualShadowMapVisualizationRecord& Record = It.Value();
		if (Record.Type == Type)
		{
			if (!bAddedCommands && bSeparatorBefore)
			{
				Menu.AddMenuSeparator();
			}
			Menu.AddMenuEntry(Record.Command, NAME_None, Record.Command->GetLabel());
			bAddedCommands = true;
		}
	}

	return bAddedCommands;
}

FVirtualShadowMapVisualizationMenuCommands::TCommandConstIterator FVirtualShadowMapVisualizationMenuCommands::CreateCommandConstIterator() const
{
	return CommandMap.CreateConstIterator();
}

void FVirtualShadowMapVisualizationMenuCommands::RegisterCommands()
{
	BuildCommandMap();
}

void FVirtualShadowMapVisualizationMenuCommands::BindCommands(FUICommandList& CommandList, const TSharedPtr<FEditorViewportClient>& Client) const
{
	// Map Virtual shadow map visualization mode actions
	for (FVirtualShadowMapVisualizationMenuCommands::TCommandConstIterator It = FVirtualShadowMapVisualizationMenuCommands::Get().CreateCommandConstIterator(); It; ++It)
	{
		const FVirtualShadowMapVisualizationMenuCommands::FVirtualShadowMapVisualizationRecord& Record = It.Value();
		CommandList.MapAction(
			Record.Command,
			FExecuteAction::CreateStatic(&FVirtualShadowMapVisualizationMenuCommands::ChangeVirtualShadowMapVisualizationMode, Client.ToWeakPtr(), Record.Name),
			FCanExecuteAction(),
			FIsActionChecked::CreateStatic(&FVirtualShadowMapVisualizationMenuCommands::IsVirtualShadowMapVisualizationModeSelected, Client.ToWeakPtr(), Record.Name)
		);
	}

	IConsoleVariable* ShowStatsVisibleCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Shadow.Virtual.Stats.Visible"));
	check(ShowStatsVisibleCVar);
	CommandList.MapAction(
		ShowStatsCommand,
		FExecuteAction::CreateLambda([=]()
		{
			ShowStatsVisibleCVar->Set(!ShowStatsVisibleCVar->GetBool());
		}),
		FCanExecuteAction::CreateLambda([]() { return true; }),
		FIsActionChecked::CreateLambda([=]() { return ShowStatsVisibleCVar->GetBool(); })
	);
	CommandList.MapAction(
		VisualizeNextLightCommand,
		FExecuteAction::CreateLambda([]()
		{
			IConsoleCommand* Command = IConsoleManager::Get().FindConsoleObject(TEXT("r.Shadow.Virtual.Visualize.NextLight"))->AsCommand();
			TArray<FString> Args;
			Command->Execute(Args, nullptr, *GLog);
		}),
		FCanExecuteAction(),
		EUIActionRepeatMode::RepeatEnabled
	);
	CommandList.MapAction(
		VisualizePrevLightCommand,
		FExecuteAction::CreateLambda([]()
		{
			IConsoleCommand* Command = IConsoleManager::Get().FindConsoleObject(TEXT("r.Shadow.Virtual.Visualize.PrevLight"))->AsCommand();
			TArray<FString> Args;
			Command->Execute(Args, nullptr, *GLog);
		}),
		FCanExecuteAction(),
		EUIActionRepeatMode::RepeatEnabled
	);
}

void FVirtualShadowMapVisualizationMenuCommands::ChangeVirtualShadowMapVisualizationMode(TWeakPtr<FEditorViewportClient> WeakClient, FName InName)
{
	if (TSharedPtr<FEditorViewportClient> Client = WeakClient.Pin())
	{
		Client->ChangeVirtualShadowMapVisualizationMode(InName);
	}
}

bool FVirtualShadowMapVisualizationMenuCommands::IsVirtualShadowMapVisualizationModeSelected(TWeakPtr<FEditorViewportClient> WeakClient, FName InName)
{
	if (TSharedPtr<FEditorViewportClient> Client = WeakClient.Pin())
	{
		return Client->IsVirtualShadowMapVisualizationModeSelected(InName);
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
