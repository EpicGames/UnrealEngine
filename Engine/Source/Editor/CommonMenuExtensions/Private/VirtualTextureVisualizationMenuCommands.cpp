// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualTextureVisualizationMenuCommands.h"

#include "EditorViewportClient.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "VT/VirtualTextureVisualizationData.h"

#define LOCTEXT_NAMESPACE "VirtualTextureVisualizationMenuCommands"

UE_DEFINE_TCOMMANDS(FVirtualTextureVisualizationMenuCommands)

FVirtualTextureVisualizationMenuCommands::FVirtualTextureVisualizationMenuCommands()
	: TCommands<FVirtualTextureVisualizationMenuCommands>
	(
		TEXT("VirtualTextureVisualizationMenu"), // Context name for fast lookup
		NSLOCTEXT("Contexts", "VirtualTextureVisualizationMenu", "VirtualTexture"), // Localized context name for displaying
		NAME_None, // Parent context name.  
		FAppStyle::GetAppStyleSetName() // Icon Style Set
	),
	CommandMap()
{
}

void FVirtualTextureVisualizationMenuCommands::BuildCommandMap()
{
	CommandMap.Empty();

	const FVirtualTextureVisualizationData& VisualizationData = GetVirtualTextureVisualizationData();
	for (FVirtualTextureVisualizationData::TModeArray::TConstIterator It = VisualizationData.GetModes().CreateConstIterator(); It; ++It)
	{
		const FVirtualTextureVisualizationData::FModeRecord& Entry = *It;
		FVisualizationRecord& Record = CommandMap.Add(Entry.ModeName, FVisualizationRecord());
		Record.Name = Entry.ModeName;
		Record.ModeID = Entry.ModeID;
		Record.Command = FUICommandInfoDecl(
			this->AsShared(),
			Entry.ModeName,
			Entry.ModeText,
			Entry.ModeDesc)
			.UserInterfaceType(EUserInterfaceActionType::RadioButton)
			.DefaultChord(FInputChord()
		);

	}
}

void FVirtualTextureVisualizationMenuCommands::BuildVisualisationSubMenu(FMenuBuilder& Menu)
{
	const FVirtualTextureVisualizationMenuCommands& Commands = FVirtualTextureVisualizationMenuCommands::Get();
	if (Commands.IsPopulated())
	{
		Commands.AddCommandTypeToMenu(Menu, EVirtualTextureVisualizationMode::PendingMips);
		Commands.AddCommandTypeToMenu(Menu, EVirtualTextureVisualizationMode::StackCount);
	}
}

bool FVirtualTextureVisualizationMenuCommands::AddCommandTypeToMenu(FMenuBuilder& Menu, const EVirtualTextureVisualizationMode ModeID) const
{
	bool bAddedCommands = false;

	const TVisualizationModeCommandMap& Commands = CommandMap;
	for (TCommandConstIterator It = CreateCommandConstIterator(); It; ++It)
	{
		const FVisualizationRecord& Record = It.Value();
		if (Record.ModeID == ModeID)
		{
			Menu.AddMenuEntry(Record.Command, NAME_None, Record.Command->GetLabel());
			bAddedCommands = true;
		}
	}

	return bAddedCommands;
}

FVirtualTextureVisualizationMenuCommands::TCommandConstIterator FVirtualTextureVisualizationMenuCommands::CreateCommandConstIterator() const
{
	return CommandMap.CreateConstIterator();
}

void FVirtualTextureVisualizationMenuCommands::RegisterCommands()
{
	BuildCommandMap();
}

void FVirtualTextureVisualizationMenuCommands::BindCommands(FUICommandList& CommandList, const TSharedPtr<FEditorViewportClient>& Client) const
{
	for (FVirtualTextureVisualizationMenuCommands::TCommandConstIterator It = FVirtualTextureVisualizationMenuCommands::Get().CreateCommandConstIterator(); It; ++It)
	{
		const FVirtualTextureVisualizationMenuCommands::FVisualizationRecord& Record = It.Value();
		CommandList.MapAction(
			Record.Command,
			FExecuteAction::CreateStatic(&FVirtualTextureVisualizationMenuCommands::ChangeVisualizationMode, Client.ToWeakPtr(), Record.Name),
			FCanExecuteAction(),
			FIsActionChecked::CreateStatic(&FVirtualTextureVisualizationMenuCommands::IsVisualizationModeSelected, Client.ToWeakPtr(), Record.Name)
		);
	}
}

void FVirtualTextureVisualizationMenuCommands::ChangeVisualizationMode(TWeakPtr<FEditorViewportClient> WeakClient, FName InName)
{
	if (TSharedPtr<FEditorViewportClient> Client = WeakClient.Pin())
	{
		Client->ChangeVirtualTextureVisualizationMode(InName);
	}
}

bool FVirtualTextureVisualizationMenuCommands::IsVisualizationModeSelected(TWeakPtr<FEditorViewportClient> WeakClient, FName InName)
{
	if (TSharedPtr<FEditorViewportClient> Client = WeakClient.Pin())
	{
		return Client->IsVirtualTextureVisualizationModeSelected(InName);
	}
	
	return false;
}

#undef LOCTEXT_NAMESPACE
