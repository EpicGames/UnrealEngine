// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

#define UE_API COMMONMENUEXTENSIONS_API

class FEditorViewportClient;
class FMenuBuilder;
class FUICommandInfo;
class FUICommandList;

UE_DECLARE_TCOMMANDS(class FBufferVisualizationMenuCommands, UE_API)

class FBufferVisualizationMenuCommands : public TCommands<FBufferVisualizationMenuCommands>
{
public:
	struct FBufferVisualizationRecord
	{
		FName Name;
		TSharedPtr<FUICommandInfo> Command;

		FBufferVisualizationRecord()
			: Name(),
			  Command()
		{
		}
	};

	typedef TMultiMap<FName, FBufferVisualizationRecord> TBufferVisualizationModeCommandMap;
	typedef TBufferVisualizationModeCommandMap::TConstIterator TCommandConstIterator;

	UE_API FBufferVisualizationMenuCommands();

	UE_API TCommandConstIterator CreateCommandConstIterator() const;
	UE_API const FBufferVisualizationRecord& OverviewCommand() const;

	static UE_API void BuildVisualisationSubMenu(FMenuBuilder& Menu);

	UE_API virtual void RegisterCommands() override;

	UE_API void BindCommands(FUICommandList& CommandList, const TSharedPtr<FEditorViewportClient>& Client) const;

private:
	void BuildCommandMap();

	void CreateOverviewCommand();
	void CreateVisualizationCommands();

	void AddOverviewCommandToMenu(FMenuBuilder& Menu) const;
	void AddVisualizationCommandsToMenu(FMenuBuilder& menu) const;

	static void ChangeBufferVisualizationMode(TWeakPtr<FEditorViewportClient> WeakClient, FName InName);
	static bool IsBufferVisualizationModeSelected(TWeakPtr<FEditorViewportClient> WeakClient, FName InName);

	TBufferVisualizationModeCommandMap CommandMap;
};

#undef UE_API