// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

#define UE_API COMMONMENUEXTENSIONS_API

class SEditorViewport;
class FEditorViewportClient;
class FMenuBuilder;
class FUICommandInfo;
class FUICommandList;
struct FToolMenuEntry;
enum class EGroomViewMode : uint8;

UE_DECLARE_TCOMMANDS(class FGroomVisualizationMenuCommands, UE_API)

class FGroomVisualizationMenuCommands : public TCommands<FGroomVisualizationMenuCommands>
{
public:
	struct FGroomVisualizationRecord
	{
		FName Name;
		TSharedPtr<FUICommandInfo> Command;
		EGroomViewMode Mode;
	};

	typedef TMultiMap<FName, FGroomVisualizationRecord> TGroomVisualizationModeCommandMap;
	typedef TGroomVisualizationModeCommandMap::TConstIterator TCommandConstIterator;

	UE_API FGroomVisualizationMenuCommands();

	UE_API TCommandConstIterator CreateCommandConstIterator() const;

	static UE_API void BuildVisualisationSubMenu(FMenuBuilder& Menu);
	static UE_API void BuildVisualisationSubMenuForGroomEditor(FMenuBuilder& Menu);
	static UE_API FToolMenuEntry BuildVisualizationSubMenuItem(const TWeakPtr<SEditorViewport>& Viewport);
	static UE_API FToolMenuEntry BuildVisualizationSubMenuItemForGroomEditor(const TWeakPtr<SEditorViewport>& Viewport);

	UE_API virtual void RegisterCommands() override;

	UE_API void BindCommands(FUICommandList& CommandList, const TSharedPtr<FEditorViewportClient>& Client) const;

	inline bool IsPopulated() const
	{
		return CommandMap.Num() > 0;
	}

private:
	void BuildCommandMap();
	bool AddCommandTypeToMenu(FMenuBuilder& Menu, const EGroomViewMode Type) const;

	static void InternalBuildVisualisationSubMenu(FMenuBuilder& Menu, bool bIsGroomEditor);
	static void ChangeGroomVisualizationMode(TWeakPtr<FEditorViewportClient> WeakClient, FName InName);
	static bool IsGroomVisualizationModeSelected(TWeakPtr<FEditorViewportClient> WeakClient, FName InName);
	
	static FToolMenuEntry InternalBuildVisualizationSubMenuItem(const TWeakPtr<SEditorViewport>& Viewport, bool bIsGroomEditor);

private:
	TGroomVisualizationModeCommandMap CommandMap;
};

#undef UE_API