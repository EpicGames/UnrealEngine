// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "UObject/NameTypes.h"
#include "Templates/SharedPointer.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Commands/Commands.h"

#define UE_API COMMONMENUEXTENSIONS_API

class FEditorViewportClient;
class UToolMenu;
struct FToolMenuSection;

UE_DECLARE_TCOMMANDS(class FRayTracingDebugVisualizationMenuCommands, UE_API)

class FRayTracingDebugVisualizationMenuCommands : public TCommands<FRayTracingDebugVisualizationMenuCommands>
{
public:
	enum class FVisualizationType : uint8
	{
		Overview,
		Standard,
		Performance,
		Timing,
		Other,
	};

	struct FVisualizationRecord
	{
		FName Name;
		TSharedPtr<FUICommandInfo> Command;
		FVisualizationType Type;

		FVisualizationRecord()
			: Name()
			, Command()
			, Type(FVisualizationType::Overview)
		{
		}
	};

	typedef TMultiMap<FName, FVisualizationRecord> TVisualizationModeCommandMap;
	typedef TVisualizationModeCommandMap::TConstIterator TCommandConstIterator;

	UE_API FRayTracingDebugVisualizationMenuCommands();

	UE_API TCommandConstIterator CreateCommandConstIterator() const;

	static UE_API void BuildVisualisationSubMenu(UToolMenu* InMenu);

	UE_API virtual void RegisterCommands() override;

	UE_API void BindCommands(FUICommandList& CommandList, const TSharedPtr<FEditorViewportClient>& Client) const;

	inline bool IsPopulated() const
	{
		return CommandMap.Num() > 0;
	}

private:
	void BuildCommandMap();

	bool AddCommandTypeToSection(FToolMenuSection& InSection, const FVisualizationType Type) const;
	bool AddLayersSection(UToolMenu& InMenu) const;
	bool AddOptionsSection(UToolMenu& InMenu) const;

	static void ChangeRayTracingDebugVisualizationMode(TWeakPtr<FEditorViewportClient> WeakClient, FName InName);
	static bool IsRayTracingDebugVisualizationModeSelected(TWeakPtr<FEditorViewportClient> WeakClient, FName InName);

	TVisualizationModeCommandMap CommandMap;

	TSharedPtr<FUICommandInfo> ShowNearFieldCommand;
	TSharedPtr<FUICommandInfo> ShowFarFieldCommand;

	TSharedPtr<FUICommandInfo> ShowOpaqueOnlyCommand;
};

#undef UE_API
