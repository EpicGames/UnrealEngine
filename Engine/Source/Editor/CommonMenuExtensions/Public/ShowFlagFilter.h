// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "EditorShowFlags.h"
#include "HAL/Platform.h"
#include "ShowFlags.h"

#define UE_API COMMONMENUEXTENSIONS_API

class FShowFlagData;

class FShowFlagFilter
{
public:
	enum EDefaultMode
	{
		IncludeAllFlagsByDefault,
		ExcludeAllFlagsByDefault
	};

	struct FGroupedShowFlagIndices
	{
		inline FGroupedShowFlagIndices()
			: GroupedIndices{}
		{
		}

		inline TArray<uint32>& operator [](EShowFlagGroup Group)
		{
			return GroupedIndices[Group];
		}

		inline const TArray<uint32>& operator [](EShowFlagGroup Group) const
		{
			return GroupedIndices[Group];
		}

		inline void ClearAll()
		{
			for ( int32 Index = 0; Index < SFG_Max; ++Index )
			{
				GroupedIndices[Index].Empty();
			}
		};

		inline int32 TotalIndices() const
		{
			int32 Count = 0;

			for (int32 Index = 0; Index < SFG_Max; ++Index)
			{
				Count += GroupedIndices[Index].Num();
			}

			return Count;
		}

	private:
		TArray<uint32> GroupedIndices[SFG_Max];
	};

	UE_API explicit FShowFlagFilter(EDefaultMode InDefaultMode);

	// Operations (earlier = higher priority):
	// - Include all explicitly included flags.
	// - Exclude all explicitly excluded flags.
	// - Include all flags in the include groups.
	// - Exclude all flags in the exclude groups.
	// - Include or exclude remaining flags depending on the default mode.
	UE_API FShowFlagFilter& IncludeFlag(FEngineShowFlags::EShowFlag Flag);
	UE_API FShowFlagFilter& ExcludeFlag(FEngineShowFlags::EShowFlag Flag);
	UE_API FShowFlagFilter& IncludeGroup(EShowFlagGroup Group);
	UE_API FShowFlagFilter& ExcludeGroup(EShowFlagGroup Group);

	UE_API const FGroupedShowFlagIndices& GetFilteredIndices(bool bForceRebuild = false) const;

private:
	UE_API bool ShouldIncludeFlag(const FShowFlagData& Flag) const;
	UE_API void RebuildIndexArray() const;

	EDefaultMode DefaultMode;

	// These don't need to be maps/sets because the number of groups is small
	// and we'll get cache benefit from the array.
	TArray<EShowFlagGroup> IncludedGroups;
	TArray<EShowFlagGroup> ExcludedGroups;

	TSet<FEngineShowFlags::EShowFlag> IncludedFlags;
	TSet<FEngineShowFlags::EShowFlag> ExcludedFlags;

	// Flags that have passed the filtering.
	// These are indices into the GetShowFlagMenuItems() array.
	mutable FGroupedShowFlagIndices AllowedFlagIndices;
	mutable bool bHaveFiltered;
};

#undef UE_API
