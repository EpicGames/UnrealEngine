// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "EditorViewportClient.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UICommandInfo.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "ShowFlagFilter.h"
#include "ShowFlags.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"

#define UE_API COMMONMENUEXTENSIONS_API

class FEditorViewportClient;
class FShowFlagData;
class FUICommandInfo;
class FUICommandList;
class UToolMenu;
struct FToolMenuSection;

UE_DECLARE_TCOMMANDS(class FShowFlagMenuCommands, UE_API)

class FShowFlagMenuCommands : public TCommands<FShowFlagMenuCommands>
{
public:
	struct FShowFlagCommand
	{
		FEngineShowFlags::EShowFlag FlagIndex;
		TSharedPtr<FUICommandInfo> ShowMenuItem;
		FText LabelOverride;

		FShowFlagCommand(FEngineShowFlags::EShowFlag InFlagIndex, const TSharedPtr<FUICommandInfo>& InShowMenuItem, const FText& InLabelOverride)
			: FlagIndex(InFlagIndex),
			  ShowMenuItem(InShowMenuItem),
			  LabelOverride(InLabelOverride)
		{
		}

		FShowFlagCommand(FEngineShowFlags::EShowFlag InFlagIndex, const TSharedPtr<FUICommandInfo>& InShowMenuItem)
			: FlagIndex(InFlagIndex),
			  ShowMenuItem(InShowMenuItem),
			  LabelOverride()
		{
		}
	};

	UE_API FShowFlagMenuCommands();

	UE_API virtual void RegisterCommands() override;

	UE_API void BindCommands(FUICommandList& CommandList, const TSharedPtr<FEditorViewportClient>& Client) const;
	UE_API void BuildShowFlagsMenu(UToolMenu* Menu, const FShowFlagFilter& Filter = FShowFlagFilter(FShowFlagFilter::IncludeAllFlagsByDefault)) const;

	UE_API void PopulateCommonShowFlagsSection(
		FToolMenuSection& Section, const FShowFlagFilter& Filter = FShowFlagFilter(FShowFlagFilter::IncludeAllFlagsByDefault)
	) const;
	UE_API void PopulateAllShowFlagsSection(
		FToolMenuSection& Section, const FShowFlagFilter& Filter = FShowFlagFilter(FShowFlagFilter::IncludeAllFlagsByDefault)
	) const;
	
	UE_API const TArray<FShowFlagCommand>& GetCommands() const;
	UE_API TSharedPtr<const FUICommandInfo> FindCommandForFlag(FEngineShowFlags::EShowFlag FlagIndex) const;

private:
	static void StaticCreateShowFlagsSubMenu(UToolMenu* Menu, TArray<uint32> FlagIndices, int32 EntryOffset);
	static void ToggleShowFlag(TWeakPtr<FEditorViewportClient> WeakClient, FEngineShowFlags::EShowFlag EngineShowFlagIndex);
	static bool IsShowFlagEnabled(TWeakPtr<FEditorViewportClient> WeakClient, FEngineShowFlags::EShowFlag EngineShowFlagIndex);

	FSlateIcon GetShowFlagIcon(const FShowFlagData& Flag) const;

	void CreateShowFlagCommands();
	void UpdateCustomShowFlags() const;
	void CreateSubMenuIfRequired(FToolMenuSection& Section, const FShowFlagFilter& Filter, EShowFlagGroup Group, const FName SubMenuName, const FText& MenuLabel, const FText& ToolTip, const FName IconName) const;
	void CreateShowFlagsSubMenu(UToolMenu* Menu, TArray<uint32> MenuCommands, int32 EntryOffset) const;

	TArray<FShowFlagCommand> ShowFlagCommands;
	bool bCommandsInitialised;
};

#undef UE_API
