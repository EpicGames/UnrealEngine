// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "SubstrateVisualizationData.h"

#define UE_API COMMONMENUEXTENSIONS_API

class FEditorViewportClient;
class FMenuBuilder;
class FUICommandInfo;
class FUICommandList;

UE_DECLARE_TCOMMANDS(class FSubstrateVisualizationMenuCommands, UE_API)

class FSubstrateVisualizationMenuCommands : public TCommands<FSubstrateVisualizationMenuCommands>
{
public:
	struct FSubstrateVisualizationRecord
	{
		FName Name;
		TSharedPtr<FUICommandInfo> Command;
		FSubstrateViewMode ViewMode = FSubstrateViewMode::MaterialProperties;
	};

	typedef TMultiMap<FName, FSubstrateVisualizationRecord> TSubstrateVisualizationModeCommandMap;
	typedef TSubstrateVisualizationModeCommandMap::TConstIterator TCommandConstIterator;

	UE_API FSubstrateVisualizationMenuCommands();

	UE_API TCommandConstIterator CreateCommandConstIterator() const;

	static UE_API void BuildVisualisationSubMenu(FMenuBuilder& Menu);

	UE_API virtual void RegisterCommands() override;

	UE_API void BindCommands(FUICommandList& CommandList, const TSharedPtr<FEditorViewportClient>& Client) const;
	
	inline bool IsPopulated() const
	{
		return CommandMap.Num() > 0;
	}

private:
	void BuildCommandMap();
	bool AddCommandTypeToMenu(FMenuBuilder& Menu, const FSubstrateViewMode ViewMode) const;

	static void ChangeSubstrateVisualizationMode(TWeakPtr<FEditorViewportClient> WeakClient, FName InName);
	static bool IsSubstrateVisualizationModeSelected(TWeakPtr<FEditorViewportClient> WeakClient, FName InName);

private:
	TSubstrateVisualizationModeCommandMap CommandMap;
};

#undef UE_API