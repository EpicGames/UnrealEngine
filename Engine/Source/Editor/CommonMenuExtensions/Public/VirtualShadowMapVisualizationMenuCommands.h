// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

#define UE_API COMMONMENUEXTENSIONS_API

class FEditorViewportClient;
class FMenuBuilder;
class FUICommandInfo;
class FUICommandList;

UE_DECLARE_TCOMMANDS(class FVirtualShadowMapVisualizationMenuCommands, UE_API)

class FVirtualShadowMapVisualizationMenuCommands : public TCommands<FVirtualShadowMapVisualizationMenuCommands>
{
public:
	// Re-sort into categories for the menu
	enum class FVirtualShadowMapVisualizationType : uint8
	{
		Standard,
		Advanced,
	};

	struct FVirtualShadowMapVisualizationRecord
	{
		FName Name;
		TSharedPtr<FUICommandInfo> Command;
		FVirtualShadowMapVisualizationType Type;

		FVirtualShadowMapVisualizationRecord()
		: Name()
		, Command()
		, Type(FVirtualShadowMapVisualizationType::Standard)
		{
		}
	};

	typedef TMultiMap<FName, FVirtualShadowMapVisualizationRecord> TVirtualShadowMapVisualizationModeCommandMap;
	typedef TVirtualShadowMapVisualizationModeCommandMap::TConstIterator TCommandConstIterator;

	UE_API FVirtualShadowMapVisualizationMenuCommands();

	UE_API TCommandConstIterator CreateCommandConstIterator() const;

	static UE_API void BuildVisualisationSubMenu(FMenuBuilder& Menu);

	UE_API virtual void RegisterCommands() override;

	UE_API void BindCommands(FUICommandList& CommandList, const TSharedPtr<FEditorViewportClient>& Client) const;
	
	inline bool IsPopulated() const
	{
		return CommandMap.Num() > 0;
	}

private:
	void BuildCommandMap();
	bool AddCommandTypeToMenu(FMenuBuilder& Menu, const FVirtualShadowMapVisualizationType Type, bool bSeparatorBefore = false) const;

	static void ChangeVirtualShadowMapVisualizationMode(TWeakPtr<FEditorViewportClient> WeakClient, FName InName);
	static bool IsVirtualShadowMapVisualizationModeSelected(TWeakPtr<FEditorViewportClient> WeakClient, FName InName);

private:
	TVirtualShadowMapVisualizationModeCommandMap CommandMap;
	TSharedPtr<FUICommandInfo> ShowStatsCommand;
	TSharedPtr<FUICommandInfo> VisualizeNextLightCommand;
	TSharedPtr<FUICommandInfo> VisualizePrevLightCommand;
};

#undef UE_API