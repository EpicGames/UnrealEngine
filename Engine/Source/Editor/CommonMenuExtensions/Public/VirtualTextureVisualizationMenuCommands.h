// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

#define UE_API COMMONMENUEXTENSIONS_API

class SEditorViewport;
class FEditorViewportClient;
class FMenuBuilder;
class FUICommandInfo;
class FUICommandList;
struct FToolMenuEntry;
enum class EVirtualTextureVisualizationMode : uint8;

UE_DECLARE_TCOMMANDS(class FVirtualTextureVisualizationMenuCommands, UE_API)

class FVirtualTextureVisualizationMenuCommands : public TCommands<FVirtualTextureVisualizationMenuCommands>
{
public:
	struct FVisualizationRecord
	{
		FName Name;
		TSharedPtr<FUICommandInfo> Command;
		EVirtualTextureVisualizationMode ModeID;
	};

	typedef TMultiMap<FName, FVisualizationRecord> TVisualizationModeCommandMap;
	typedef TVisualizationModeCommandMap::TConstIterator TCommandConstIterator;

	UE_API FVirtualTextureVisualizationMenuCommands();

	UE_API TCommandConstIterator CreateCommandConstIterator() const;

	static UE_API void BuildVisualisationSubMenu(FMenuBuilder& Menu);

	UE_API virtual void RegisterCommands() override;

	UE_API void BindCommands(FUICommandList& CommandList, const TSharedPtr<FEditorViewportClient>& Client) const;

	inline bool IsPopulated() const
	{
		return CommandMap.Num() > 0;
	}

private:
	void BuildCommandMap();
	bool AddCommandTypeToMenu(FMenuBuilder& Menu, const EVirtualTextureVisualizationMode ModeID) const;

	static void ChangeVisualizationMode(TWeakPtr<FEditorViewportClient> WeakClient, FName InName);
	static bool IsVisualizationModeSelected(TWeakPtr<FEditorViewportClient> WeakClient, FName InName);
	
private:
	TVisualizationModeCommandMap CommandMap;
};

#undef UE_API